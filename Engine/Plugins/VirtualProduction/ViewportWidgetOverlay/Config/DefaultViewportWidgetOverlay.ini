[CoreRedirects]

; 5.7 > 5.8
+ClassRedirects=(OldName="VPFullScreenUserWidget",NewName="/Script/ViewportWidgetOverlay.ViewportWidgetOverlay")
+StructRedirects=(OldName="/Script/VPUtilities.VPFullScreenUserWidget_PostProcess", NewName="/Script/ViewportWidgetOverlay.ViewportWidgetOverlay_PostProcess")
+StructRedirects=(OldName="/Script/VPUtilities.VPFullScreenUserWidget_PostProcessBase", NewName="/Script/ViewportWidgetOverlay.ViewportWidgetOverlay_PostProcessBase")
+StructRedirects=(OldName="/Script/VPUtilities.VPFullScreenUserWidget_PostProcessWithSVE", NewName="/Script/ViewportWidgetOverlay.ViewportWidgetOverlay_PostProcessWithSVE")
+EnumRedirects=(OldName="/Script/VPUtilities.EVPWidgetDisplayType", NewName="/Script/ViewportWidgetOverlay.EViewportWidgetOverlay_DisplayType"))
+PackageRedirects=(OldName="/VirtualProductionUtilities/Materials/WidgetPostProcessMaterial",NewName="/ViewportWidgetOverlay/Materials/WidgetPostProcessMaterial")