// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 ViewportWidgetOverlay.usf: Shader for overlaying a widget over another texture
 =============================================================================*/
 
#include "/Engine/Public/Platform.ush"

/* Pixel shaders
=============================================================================*/

// The render target containing the result of FWidgetRenderer::DrawWindow
Texture2D WidgetTexture;
SamplerState WidgetSampler;
// This is the scene color texture, retrieved by the SVE
Texture2D SceneTexture;
SamplerState SceneSampler;

void OverlayWidgetPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	float4 WidgetColor = WidgetTexture.Sample(WidgetSampler, UV);
	float4 SceneColor = SceneTexture.Sample(SceneSampler, UV);
	// Blending using the widget texture alpha value - this effectively overlays the widget
	OutColor.rgba = float4(WidgetColor.rgb * WidgetColor.a + SceneColor.rgb * (1 - WidgetColor.a), SceneColor.a);
}


