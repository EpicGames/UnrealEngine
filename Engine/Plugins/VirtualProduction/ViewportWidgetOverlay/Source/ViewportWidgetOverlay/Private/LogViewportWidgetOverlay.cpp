// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogViewportWidgetOverlay.h"

DEFINE_LOG_CATEGORY(LogViewportWidgetOverlay);