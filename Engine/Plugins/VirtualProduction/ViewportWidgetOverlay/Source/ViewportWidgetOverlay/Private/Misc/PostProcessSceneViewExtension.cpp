// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcessSceneViewExtension.h"

#include "CommonRenderResources.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PostProcess/DrawRectangle.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "ScreenPass.h"
#include "TextureResource.h"
#include "SceneView.h"
#include "GlobalShader.h"
#include "RenderGraph.h"
#include "RHI.h"

namespace UE::ViewportWidgetOverlay::Private
{
FPostProcessSceneViewExtension::FPostProcessSceneViewExtension(const FAutoRegister& AutoRegister, UTextureRenderTarget2D& WidgetRenderTarget)
	: Super(AutoRegister)
	, WidgetRenderTarget(&WidgetRenderTarget)
{}

void FPostProcessSceneViewExtension::PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder,FSceneViewFamily& InViewFamily)
{
	if (!WidgetRenderTarget.IsValid())
	{
		return;
	}

	FRDGTextureRef ViewFamilyTexture = TryCreateViewFamilyTexture(GraphBuilder, InViewFamily);
	if (!ViewFamilyTexture)
	{
		return;
	}

	for (int32 ViewIndex = 0; ViewIndex < InViewFamily.Views.Num(); ++ViewIndex)
	{
		RenderMaterial_RenderThread(GraphBuilder, *InViewFamily.Views[ViewIndex], ViewFamilyTexture);
	}
}

bool FPostProcessSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	return WidgetRenderTarget.IsValid();
}

/**
 * This shaders overlays WidgetTexture of SceneTexture by blending it like so
 *	color = WidgetTexture.A * WidgetTexture.RGB + (1 - WidgetTexture.A) * SceneTexture.RGB
 */
class FDrawTextureInShaderPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDrawTextureInShaderPS);
	SHADER_USE_PARAMETER_STRUCT(FDrawTextureInShaderPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WidgetTexture) 
		SHADER_PARAMETER_SAMPLER(SamplerState, WidgetSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneTexture) 
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{}
};
IMPLEMENT_GLOBAL_SHADER(FDrawTextureInShaderPS, "/Plugin/ViewportWidgetOverlay/Private/ViewportWidgetOverlay.usf", "OverlayWidgetPS", SF_Pixel);

void FPostProcessSceneViewExtension::RenderMaterial_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, FRDGTextureRef ViewFamilyTexture)
{
	// Can be invalidated after exiting PIE
	if (!WidgetRenderTarget.IsValid() || !WidgetRenderTarget->GetRenderTargetResource() || !WidgetRenderTarget->GetRenderTargetResource()->GetTexture2DRHI())
	{
		return;
	}
	
	const FTextureRHIRef WidgetRenderTarget_RHI = WidgetRenderTarget->GetRenderTargetResource()->GetTexture2DRHI();
	const FRDGTextureRef WidgetRenderTarget_RDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(WidgetRenderTarget_RHI, TEXT("WidgetRenderTarget")));

	const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	const TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);
	const TShaderMapRef<FDrawTextureInShaderPS> PixelShader(GlobalShaderMap);
	
	FDrawTextureInShaderPS::FParameters* Parameters = GraphBuilder.AllocParameters<FDrawTextureInShaderPS::FParameters>();
	Parameters->WidgetTexture = WidgetRenderTarget_RDG;
	Parameters->WidgetSampler = TStaticSamplerState<SF_Point>::GetRHI();
	Parameters->SceneTexture = ViewFamilyTexture;
	Parameters->SceneSampler = TStaticSamplerState<SF_Point>::GetRHI();
	Parameters->RenderTargets[0] = FRenderTargetBinding{ ViewFamilyTexture, ERenderTargetLoadAction::ELoad };

	const FScreenPassTextureViewport InputViewport(WidgetRenderTarget_RDG);
	const FScreenPassTextureViewport OutputViewport(ViewFamilyTexture);
	// Note that we reference WidgetRenderTarget here, which means RDG will synchronize access to it.
	// That means that the DrawWindow operation (see FVPFullScreenUserWidget_PostProcessBase) will finish writing into WidgetRenderTarget before we access it with this pass.
	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("VPFullScreenPostProcessOverlay"),
		InView,
		OutputViewport,
		InputViewport,
		VertexShader,
		PixelShader,
		Parameters
		);
}
}
