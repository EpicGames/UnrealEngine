// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"
#include "UObject/WeakObjectPtrFwd.h"

class UMaterialInterface;
class UTextureRenderTarget2D;

namespace UE::ViewportWidgetOverlay::Private
{
	/**
	 * Renders a post process material over the entirety of the screen.
	 * 
	 * This method differs from adding a post process material as blend material to a cine camera by
	 * ignoring anamorphic squeeze. If the Anamorphic Squeeze == 1, both methods produce the same output; if anamorphic
	 * squeeze != 1, the cine camera will smallen the viewport vertically (adding black bars to the top and bottom) and render the blend material.
	 * This method will render the post process material over the entire area of the viewport (over the black bars as well).
	 *
	 * Intended to be used with a post process material that renders a widget over the viewport, e.g. useful for pixel streaming.
	 */
	class FPostProcessSceneViewExtension : public FSceneViewExtensionBase
	{
		using Super = FSceneViewExtensionBase;
	public:
		
		FPostProcessSceneViewExtension(const FAutoRegister& AutoRegister, UTextureRenderTarget2D& WidgetRenderTarget);

		//~ISceneViewExtension interface
		virtual void PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
		virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
		//~ISceneViewExtension interface

	private:
		
		/** Contains the widget that is supposed to be overlaid. */
		TWeakObjectPtr<UTextureRenderTarget2D> WidgetRenderTarget;
		
		void RenderMaterial_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, FRDGTextureRef ViewFamilyTexture);
	};
}

