// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ViewportWidgetOverlay_PostProcess.h"

#include "Components/PostProcessComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/Package.h"
#include "ViewportWidgetOverlay.h"
#include "Widgets/Layout/SConstraintCanvas.h"

#if WITH_EDITOR
#include "LevelEditor.h"
#include "SLevelViewport.h"
#endif

namespace UE::VPUtilities::Private
{
const FName NAME_SlateUI = "SlateUI";
const FName NAME_TintColorAndOpacity = "TintColorAndOpacity";
const FName NAME_OpacityFromTexture = "OpacityFromTexture";
}

void FViewportWidgetOverlay_PostProcess::SetCustomPostProcessSettingsSource(TWeakObjectPtr<UObject> InCustomPostProcessSettingsSource)
{
	CustomPostProcessSettingsSource = InCustomPostProcessSettingsSource;
	
	const bool bIsRunning = PostProcessMaterialInstance != nullptr;
	if (bIsRunning)
	{
		// Save us from creating another struct member: PostProcessMaterialInstance is always created with UWorld as outer.
		UWorld* World = CastChecked<UWorld>(PostProcessMaterialInstance->GetOuter());
		InitPostProcessComponent(World);
	}
}

bool FViewportWidgetOverlay_PostProcess::Display(UWorld* World, UUserWidget* Widget, TAttribute<float> InDPIScale)
{
	return CreateRenderer(World, Widget, MoveTemp(InDPIScale)) && InitPostProcessComponent(World);
}

void FViewportWidgetOverlay_PostProcess::Hide(UWorld* World)
{
	ReleasePostProcessComponent();
	FViewportWidgetOverlay_PostProcessBase::Hide(World);
}

void FViewportWidgetOverlay_PostProcess::Tick(UWorld* World, float DeltaSeconds)
{
	TickRenderer(World, DeltaSeconds);
}

bool FViewportWidgetOverlay_PostProcess::InitPostProcessComponent(UWorld* World)
{
	ReleasePostProcessComponent();
	if (World && ensureMsgf(PostProcessMaterial, TEXT("Was supposed to have been inited by base class")))
	{
		const bool bUseExternalPostProcess = CustomPostProcessSettingsSource.IsValid();
		if (!bUseExternalPostProcess)
		{
			AWorldSettings* WorldSetting = World->GetWorldSettings();
			PostProcessComponent = NewObject<UPostProcessComponent>(WorldSetting, NAME_None, RF_Transient);
			PostProcessComponent->bEnabled = true;
			PostProcessComponent->bUnbound = true;
			PostProcessComponent->RegisterComponent();
		}

		return UpdateTargetPostProcessSettingsWithMaterial();
	}

	return false;
}

bool FViewportWidgetOverlay_PostProcess::UpdateTargetPostProcessSettingsWithMaterial()
{
	UMaterialInstanceDynamic* MaterialInstance = PostProcessMaterialInstance;
	if (FPostProcessSettings* const PostProcessSettings = GetPostProcessSettings()
		; PostProcessSettings && ensure(MaterialInstance))
	{
		// User added blend material should not affect the widget so insert the material at the beginning
		const FWeightedBlendable Blendable{ 1.f, MaterialInstance };
			
		// Use case: Virtual Camera specifies an external post process settings
		// 1. Virtual Camera is activated > creates this widget
		// 2. Save Map
		// 3. Reload map > we'll have an empty slot in the blendables because PostProcessMaterialInstance is transient
		const bool bReuseOldEmptySlot = PostProcessSettings->WeightedBlendables.Array.Num() > 0 && !PostProcessSettings->WeightedBlendables.Array[0].Object;
		if (bReuseOldEmptySlot)
		{
			PostProcessSettings->WeightedBlendables.Array[0].Object = MaterialInstance;
		}
		else
		{
			PostProcessSettings->WeightedBlendables.Array.Insert(Blendable, 0);
		}
		return true;
	}

	return false;
}

void FViewportWidgetOverlay_PostProcess::ReleasePostProcessComponent()
{
	const bool bIsRunning = PostProcessMaterialInstance != nullptr;
	if (!bIsRunning)
	{
		return;
	}
	
	const bool bNeedsToResetExternalSettings = CustomPostProcessSettingsSource.IsValid();
	if (FPostProcessSettings* Settings = GetPostProcessSettings()
		; bNeedsToResetExternalSettings && Settings)
	{
		const int32 Index = Settings->WeightedBlendables.Array.IndexOfByPredicate([this](const FWeightedBlendable& Blendable){ return Blendable.Object == PostProcessMaterialInstance; });
		if (Index != INDEX_NONE)
		{
			Settings->WeightedBlendables.Array.RemoveAt(Index);
		}
	}
	// CustomPostProcessSettingsSource may have gone stale
	else if (PostProcessComponent)
	{
		PostProcessComponent->UnregisterComponent();
	}
	
	PostProcessComponent = nullptr;
}

bool FViewportWidgetOverlay_PostProcess::OnRenderTargetInited()
{
	// Outer needs to be transient package: otherwise we cause a world memory leak using "Save Current Level As" due to reference not getting replaced correctly
	PostProcessMaterialInstance = UMaterialInstanceDynamic::Create(PostProcessMaterial, GetTransientPackage());
	PostProcessMaterialInstance->SetFlags(RF_Transient);
	if (ensure(PostProcessMaterialInstance))
	{
		using namespace UE::VPUtilities::Private;
		PostProcessMaterialInstance->SetTextureParameterValue(NAME_SlateUI, WidgetRenderTarget);
		PostProcessMaterialInstance->SetVectorParameterValue(NAME_TintColorAndOpacity, PostProcessTintColorAndOpacity);
		PostProcessMaterialInstance->SetScalarParameterValue(NAME_OpacityFromTexture, PostProcessOpacityFromTexture);
		return true;
	}
	return false;
}

FPostProcessSettings* FViewportWidgetOverlay_PostProcess::GetPostProcessSettings() const
{
	if (PostProcessComponent)
	{
		return &PostProcessComponent->Settings;
	}

	if (!CustomPostProcessSettingsSource.IsValid())
	{
		return nullptr;
	}

	// The easiest way without overcomplicating the API with an additional callback would be to look for the first struct property.
	// We could always extend our API to accept a callback that extracts the FPostProcessSettings from the UObject instead.
	for (TFieldIterator<FStructProperty> StructIt(CustomPostProcessSettingsSource.Get()->GetClass()); StructIt; ++StructIt)
	{
		if (StructIt->Struct == FPostProcessSettings::StaticStruct())
		{
			return StructIt->ContainerPtrToValuePtr<FPostProcessSettings>(CustomPostProcessSettingsSource.Get());
		}
	}

	return nullptr;
}
