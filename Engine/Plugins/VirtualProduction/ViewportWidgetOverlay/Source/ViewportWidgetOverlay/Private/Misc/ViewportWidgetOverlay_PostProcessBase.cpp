// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ViewportWidgetOverlay_PostProcessBase.h"

#include "LogViewportWidgetOverlay.h"
#include "Components/PostProcessComponent.h"
#include "Components/WidgetComponent.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/UserInterfaceSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Input/HittestGrid.h"
#include "Layout/Visibility.h"
#include "RHI.h"
#include "Slate/SceneViewport.h"
#include "Slate/WidgetRenderer.h"
#include "UObject/Package.h"
#include "ViewportWidgetOverlay.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/SViewport.h"

#if WITH_EDITOR
#include "LevelEditor.h"
#include "SLevelViewport.h"
#endif

namespace UE::VPUtilities::Private
{
EVisibility ConvertWindowVisibilityToVisibility(EWindowVisibility visibility)
{
	switch (visibility)
	{
	case EWindowVisibility::Visible:
		return EVisibility::Visible;
	case EWindowVisibility::SelfHitTestInvisible:
		return EVisibility::SelfHitTestInvisible;
	default:
		checkNoEntry();
		return EVisibility::SelfHitTestInvisible;
	}
}

class FVPWidgetPostProcessHitTester : public ICustomHitTestPath
{
public:
	FVPWidgetPostProcessHitTester(UWorld* InWorld, TSharedPtr<SVirtualWindow> InSlateWindow, TAttribute<float> GetDPIAttribute)
		: World(InWorld)
		, VirtualSlateWindow(InSlateWindow)
		, GetDPIAttribute(MoveTemp(GetDPIAttribute))
		, WidgetDrawSize(FIntPoint::ZeroValue)
		, LastLocalHitLocation(FVector2D::ZeroVector)
	{}

	virtual TArray<FWidgetAndPointer> GetBubblePathAndVirtualCursors(const FGeometry& InGeometry, FVector2D DesktopSpaceCoordinate, bool bIgnoreEnabledStatus) const override
	{
		// Get the list of widget at the requested location.
		TArray<FWidgetAndPointer> ArrangedWidgets;
		if (TSharedPtr<SVirtualWindow> SlateWindowPin = VirtualSlateWindow.Pin())
		{
			// For some reason the DPI is not applied correctly so we need to multiply the window's native DPI ourselves.
			// This is the setting you'd find in Windows Settings > Display > Scale and layout
			// If this bit is skipped, then hovering widgets towards the bottom right will not work
			// if system scale is > 100% AND the viewport size is not fixed (default).
			const float DPI = GetDPIAttribute.Get();
			const FVector2D LocalMouseCoordinate = DPI * InGeometry.AbsoluteToLocal(DesktopSpaceCoordinate);
			
			constexpr float CursorRadius = 0.f;
			ArrangedWidgets = SlateWindowPin->GetHittestGrid().GetBubblePath(LocalMouseCoordinate, CursorRadius, bIgnoreEnabledStatus);

			const FVirtualPointerPosition VirtualMouseCoordinate(LocalMouseCoordinate, LastLocalHitLocation);
			LastLocalHitLocation = LocalMouseCoordinate;
			for (FWidgetAndPointer& ArrangedWidget : ArrangedWidgets)
			{
				ArrangedWidget.SetPointerPosition(VirtualMouseCoordinate);
			}
		}

		return ArrangedWidgets;
	}

	virtual void ArrangeCustomHitTestChildren(FArrangedChildren& ArrangedChildren) const override
	{
		// Add the displayed slate to the list of widgets.
		if (TSharedPtr<SVirtualWindow> SlateWindowPin = VirtualSlateWindow.Pin())
		{
			FGeometry WidgetGeom;
			ArrangedChildren.AddWidget(FArrangedWidget(SlateWindowPin.ToSharedRef(), WidgetGeom.MakeChild(WidgetDrawSize, FSlateLayoutTransform())));
		}
	}

	virtual TOptional<FVirtualPointerPosition> TranslateMouseCoordinateForCustomHitTestChild(const SWidget& ChildWidget, const FGeometry& MyGeometry, const FVector2D ScreenSpaceMouseCoordinate, const FVector2D LastScreenSpaceMouseCoordinate) const override
	{
		return TOptional<FVirtualPointerPosition>();
	}

	void SetWidgetDrawSize(FIntPoint NewWidgetDrawSize)
	{
		WidgetDrawSize = NewWidgetDrawSize;
	}

private:
	TWeakObjectPtr<UWorld> World;
	TWeakPtr<SVirtualWindow> VirtualSlateWindow;
	TAttribute<float> GetDPIAttribute;
	FIntPoint WidgetDrawSize;
	mutable FVector2D LastLocalHitLocation;
};
}

FViewportWidgetOverlay_PostProcessBase::FViewportWidgetOverlay_PostProcessBase()
	: PostProcessMaterial(nullptr)
	, PostProcessTintColorAndOpacity(FLinearColor::White)
	, PostProcessOpacityFromTexture(1.0f)
	, bUseWidgetDrawSize(false)
	, WidgetDrawSize(FIntPoint(640, 360))
	, bWindowFocusable(true)
	, WindowVisibility(EWindowVisibility::SelfHitTestInvisible)
	, bReceiveHardwareInput(false)
	, RenderTargetBackgroundColor(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f))
	, RenderTargetBlendMode(EWidgetBlendMode::Masked)
	, WidgetRenderTarget(nullptr)
	, WidgetRenderer(nullptr)
	, CurrentWidgetDrawSize(FIntPoint::ZeroValue)
{}

void FViewportWidgetOverlay_PostProcessBase::Hide(UWorld* World)
{
	ReleaseRenderer();
}

TSharedPtr<SVirtualWindow> FViewportWidgetOverlay_PostProcessBase::GetSlateWindow() const
{
	return SlateWindow;
}

bool FViewportWidgetOverlay_PostProcessBase::CreateRenderer(UWorld* World, UUserWidget* Widget, TAttribute<float> InDPIScale)
{
	ReleaseRenderer();

	if (World && Widget)
	{
		constexpr bool bApplyGammaCorrection = true;
		WidgetRenderer = new FWidgetRenderer(bApplyGammaCorrection);
		WidgetRenderer->SetIsPrepassNeeded(true);
		
		// CalculateWidgetDrawSize may sometimes return {0,0}, e.g. right after engine startup when viewport not yet initialized.
		// TickRenderer will call Resize automatically once CurrentWidgetDrawSize is updated to be non-zero.
		checkf(CurrentWidgetDrawSize == FIntPoint::ZeroValue, TEXT("Expected ReleaseRenderer to reset CurrentWidgetDrawSize."));
		SlateWindow = SNew(SVirtualWindow).Size(CurrentWidgetDrawSize);
		SlateWindow->SetIsFocusable(bWindowFocusable);
		SlateWindow->SetVisibility(UE::VPUtilities::Private::ConvertWindowVisibilityToVisibility(WindowVisibility));
		SlateWindow->SetContent(
			SNew(SDPIScaler)
			.DPIScale(InDPIScale)
			[
				Widget->TakeWidget()
			]
		);

		RegisterHitTesterWithViewport(World);

		if (!Widget->IsDesignTime() && World->IsGameWorld())
		{
			UGameInstance* GameInstance = World->GetGameInstance();
			UGameViewportClient* GameViewportClient = GameInstance ? GameInstance->GetGameViewportClient() : nullptr;
			if (GameViewportClient)
			{
				SlateWindow->AssignParentWidget(GameViewportClient->GetGameViewportWidget());
			}
		}

		FLinearColor ActualBackgroundColor = RenderTargetBackgroundColor;
		switch (RenderTargetBlendMode)
		{
		case EWidgetBlendMode::Opaque:
			ActualBackgroundColor.A = 1.0f;
			break;
		case EWidgetBlendMode::Masked:
			ActualBackgroundColor.A = 0.0f;
			break;
		case EWidgetBlendMode::Transparent: break;
		default: ;
		}

		// Skip InitCustomFormat call because CalculateWidgetDrawSize may sometimes return {0,0}, e.g. right after engine startup when viewport not yet initialized
		// TickRenderer will call InitCustomFormat automatically once CurrentWidgetDrawSize is updated to be non-zero.
		checkf(CurrentWidgetDrawSize == FIntPoint::ZeroValue, TEXT("Expected ReleaseRenderer to reset CurrentWidgetDrawSize."));
		// Outer needs to be transient package: otherwise we cause a world memory leak using "Save Current Level As" due to reference not getting replaced correctly
		WidgetRenderTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);
		WidgetRenderTarget->ClearColor = ActualBackgroundColor;

		return WidgetRenderer && WidgetRenderTarget && OnRenderTargetInited();
	}

	return false;
}

void FViewportWidgetOverlay_PostProcessBase::ReleaseRenderer()
{
	if (WidgetRenderer)
	{
		BeginCleanup(WidgetRenderer);
		WidgetRenderer = nullptr;
	}
	UnRegisterHitTesterWithViewport();

	SlateWindow.Reset();
	WidgetRenderTarget = nullptr;
	CurrentWidgetDrawSize = FIntPoint::ZeroValue;
}

void FViewportWidgetOverlay_PostProcessBase::TickRenderer(UWorld* World, float DeltaSeconds)
{
	check(World);
	if (WidgetRenderTarget)
	{
		const float DrawScale = 1.0f;

		const FIntPoint NewCalculatedWidgetSize = CalculateWidgetDrawSize(World);
		if (NewCalculatedWidgetSize != CurrentWidgetDrawSize)
		{
			if (IsTextureSizeValid(NewCalculatedWidgetSize))
			{
				CurrentWidgetDrawSize = NewCalculatedWidgetSize;
				WidgetRenderTarget->InitCustomFormat(CurrentWidgetDrawSize.X, CurrentWidgetDrawSize.Y, PF_B8G8R8A8, false);
				WidgetRenderTarget->UpdateResourceImmediate();
				SlateWindow->Resize(CurrentWidgetDrawSize);
				if (CustomHitTestPath)
				{
					CustomHitTestPath->SetWidgetDrawSize(CurrentWidgetDrawSize);
				}
			}
			else
			{
				Hide(World);
			}
		}

		if (WidgetRenderer && CurrentWidgetDrawSize != FIntPoint::ZeroValue)
		{
			WidgetRenderer->DrawWindow(
				WidgetRenderTarget,
				SlateWindow->GetHittestGrid(),
				SlateWindow.ToSharedRef(),
				DrawScale,
				CurrentWidgetDrawSize,
				DeltaSeconds
				);
		}
	}
}

FIntPoint FViewportWidgetOverlay_PostProcessBase::CalculateWidgetDrawSize(UWorld* World)
{
	if (bUseWidgetDrawSize)
	{
		return WidgetDrawSize;
	}

	if (World->WorldType == EWorldType::Game || World->WorldType == EWorldType::PIE)
	{
		if (UGameViewportClient* ViewportClient = World->GetGameViewport())
		{
			// The viewport maybe resizing or not yet initialized.
			// See TickRenderer(), it will be resize on the next tick to the proper size.
			// We initialized all the rendering with an small size.
			const float SmallWidgetSize = 16.f;
			FVector2D OutSize = FVector2D(SmallWidgetSize, SmallWidgetSize);
			ViewportClient->GetViewportSize(OutSize);
			if (OutSize.X < SMALL_NUMBER)
			{
				OutSize = FVector2D(SmallWidgetSize, SmallWidgetSize);
			}
			return OutSize.IntPoint();
		}
		
		UE_LOGF(LogViewportWidgetOverlay, Warning, "CalculateWidgetDrawSize failed for game world.");
		return FIntPoint::ZeroValue;
	}

#if WITH_EDITOR
	if (const TSharedPtr<FSceneViewport> SharedActiveViewport = EditorTargetViewport.Pin())
	{
		return SharedActiveViewport->GetSize();
	}
	UE_LOGF(LogViewportWidgetOverlay, Warning, "CalculateWidgetDrawSize failed for editor world.");
#endif
	
	return FIntPoint::ZeroValue;
}

bool FViewportWidgetOverlay_PostProcessBase::IsTextureSizeValid(FIntPoint Size) const
{
	const int32 MaxAllowedDrawSize = GetMax2DTextureDimension();
	return Size.X > 0 && Size.Y > 0 && Size.X <= MaxAllowedDrawSize && Size.Y <= MaxAllowedDrawSize;
}

void FViewportWidgetOverlay_PostProcessBase::RegisterHitTesterWithViewport(UWorld* World)
{
	if (!bReceiveHardwareInput && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().RegisterVirtualWindow(SlateWindow.ToSharedRef());
	}

	const TSharedPtr<SViewport> EngineViewportWidget = GetViewport(World);
	if (EngineViewportWidget && bReceiveHardwareInput)
	{
		if (EngineViewportWidget->GetCustomHitTestPath())
		{
			UE_LOGF(LogViewportWidgetOverlay, Warning, "Can't register a hit tester for FullScreenUserWidget. There is already one defined.");
		}
		else
		{
			ViewportWidget = EngineViewportWidget;
			CustomHitTestPath = MakeShared<UE::VPUtilities::Private::FVPWidgetPostProcessHitTester>(
				World,
				SlateWindow,
				TAttribute<float>::CreateRaw(this, &FViewportWidgetOverlay_PostProcess::GetDPIScaleForPostProcessHitTester, TWeakObjectPtr<UWorld>(World))
				);
			CustomHitTestPath->SetWidgetDrawSize(CurrentWidgetDrawSize);
			EngineViewportWidget->SetCustomHitTestPath(CustomHitTestPath);
		}
	}
}

void FViewportWidgetOverlay_PostProcessBase::UnRegisterHitTesterWithViewport()
{
	if (SlateWindow.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterVirtualWindow(SlateWindow.ToSharedRef());
	}

	if (TSharedPtr<SViewport> ViewportWidgetPin = ViewportWidget.Pin())
	{
		if (ViewportWidgetPin->GetCustomHitTestPath() == CustomHitTestPath)
		{
			ViewportWidgetPin->SetCustomHitTestPath(nullptr);
		}
	}

	ViewportWidget.Reset();
	CustomHitTestPath.Reset();
}

TSharedPtr<SViewport> FViewportWidgetOverlay_PostProcessBase::GetViewport(UWorld* World) const
{
	if (World->WorldType == EWorldType::Game || World->WorldType == EWorldType::PIE)
	{
		return GEngine->GetGameViewportWidget();
	}

#if WITH_EDITOR
	if (const TSharedPtr<FSceneViewport> TargetViewportPin = EditorTargetViewport.Pin())
	{
		return TargetViewportPin->GetViewportWidget().Pin();
	}
#endif
	
	return nullptr;
}

float FViewportWidgetOverlay_PostProcessBase::GetDPIScaleForPostProcessHitTester(TWeakObjectPtr<UWorld> World) const
{
	FSceneViewport* Viewport = nullptr;
	if (ensure(World.IsValid()) && World->IsGameWorld())
	{
		UGameViewportClient* ViewportClient = World->GetGameViewport();
		Viewport = ensure(ViewportClient) ? ViewportClient->GetGameViewport() : nullptr;
	}

#if WITH_EDITOR
	const TSharedPtr<FSceneViewport> ViewportPin = EditorTargetViewport.Pin();
	Viewport = Viewport ? Viewport : ViewportPin.Get();
#endif

	const bool bCanScale = Viewport && !Viewport->HasFixedSize();
	if (!bCanScale)
	{
		return 1.f;
	}
	
	// For some reason the DPI is not applied correctly when the viewport has a fixed size and the system scale is > 100%.
	// This is the setting you'd find in Windows Settings > Display > Scale and layout
	// If this bit is skipped, then hovering widgets towards the bottom right will not work
	// if system scale is > 100% AND the viewport size is not fixed (default).
	const TSharedPtr<SWindow> ViewportWindow = Viewport->FindWindow();
	return ViewportWindow ? ViewportWindow->GetDPIScaleFactor() : 1.f;
}