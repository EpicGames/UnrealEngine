// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ViewportWidgetOverlay_PostProcessWithSVE.h"

#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "PostProcessSceneViewExtension.h"
#include "SceneViewExtension.h"
#include "ViewportWidgetOverlay.h"

bool FViewportWidgetOverlay_PostProcessWithSVE::Display(UWorld* World, UUserWidget* Widget, TAttribute<float> InDPIScale)
{
	bool bOk = CreateRenderer(World, Widget, MoveTemp(InDPIScale));
	if (bOk && ensureMsgf(WidgetRenderTarget, TEXT("CreateRenderer returned true even though it failed.")))
	{
		SceneViewExtension = FSceneViewExtensions::NewExtension<UE::ViewportWidgetOverlay::Private::FPostProcessSceneViewExtension>(
			*WidgetRenderTarget
			);
		SceneViewExtension->IsActiveThisFrameFunctions = MoveTemp(IsActiveFunctorsToRegister);
	}
	return bOk;
}

void FViewportWidgetOverlay_PostProcessWithSVE::Hide(UWorld* World)
{
	SceneViewExtension.Reset();
	FViewportWidgetOverlay_PostProcessBase::Hide(World);
}

void FViewportWidgetOverlay_PostProcessWithSVE::Tick(UWorld* World, float DeltaSeconds)
{
	TickRenderer(World, DeltaSeconds);
}

void FViewportWidgetOverlay_PostProcessWithSVE::RegisterIsActiveFunctor(FSceneViewExtensionIsActiveFunctor IsActiveFunctor)
{
	if (SceneViewExtension)
	{
		SceneViewExtension->IsActiveThisFrameFunctions.Emplace(MoveTemp(IsActiveFunctor));
	}
	else
	{
		IsActiveFunctorsToRegister.Emplace(MoveTemp(IsActiveFunctor));
	}
}
