// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportWidgetOverlay.h"

#include "LogViewportWidgetOverlay.h"
#include "Components/PostProcessComponent.h"
#include "Engine/GameViewportClient.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/UserInterfaceSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Materials/MaterialInterface.h"
#include "RHI.h"
#include "Slate/SceneViewport.h"
#include "Slate/WidgetRenderer.h"
#include "UObject/ConstructorHelpers.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SDPIScaler.h"

#if WITH_EDITOR
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "SLevelViewport.h"
#endif

#define LOCTEXT_NAMESPACE "VPFullScreenUserWidget"

/////////////////////////////////////////////////////
// Internal helper
namespace
{
const FName NAME_LevelEditorName = "LevelEditor";

namespace VPFullScreenUserWidgetPrivate
{
	/**
	 * Class made to handle world cleanup and hide/cleanup active UserWidget to avoid touching public headers
	 */
	class FWorldCleanupListener
	{
	public:

		static FWorldCleanupListener* Get()
		{
			static FWorldCleanupListener Instance;
			return &Instance;
		}

		/** Disallow Copying / Moving */
		UE_NONCOPYABLE(FWorldCleanupListener);

		~FWorldCleanupListener()
		{
			FWorldDelegates::OnWorldCleanup.RemoveAll(this);
		}

		void AddWidget(UViewportWidgetOverlay* InWidget)
		{
			WidgetsToHide.AddUnique(InWidget);
		}

		void RemoveWidget(UViewportWidgetOverlay* InWidget)
		{
			WidgetsToHide.RemoveSingleSwap(InWidget, EAllowShrinking::No);
		}

	private:

		FWorldCleanupListener()
		{
			FWorldDelegates::OnWorldCleanup.AddRaw(this, &FWorldCleanupListener::OnWorldCleanup);
		}

		void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
		{
			for (auto WeakWidgetIter = WidgetsToHide.CreateIterator(); WeakWidgetIter; ++WeakWidgetIter)
			{
				TWeakObjectPtr<UViewportWidgetOverlay>& WeakWidget = *WeakWidgetIter;
				if (UViewportWidgetOverlay* Widget = WeakWidget.Get())
				{
					if (Widget->IsDisplayed()
						&& Widget->GetWidget()
						&& (Widget->GetWidget()->GetWorld() == InWorld))
					{
						//Remove first since Hide removes object from the list
						WeakWidgetIter.RemoveCurrent();
						Widget->Hide();
					}
				}
				else
				{
					WeakWidgetIter.RemoveCurrent();
				}
			}
		}

	private:

		TArray<TWeakObjectPtr<UViewportWidgetOverlay>> WidgetsToHide;
	};
}
}

/////////////////////////////////////////////////////
// FViewportWidgetOverlay_Viewport

bool FViewportWidgetOverlay_Viewport::Display(UWorld* World, UUserWidget* Widget, TAttribute<float> InDPIScale)
{
	const TSharedPtr<SConstraintCanvas> FullScreenWidgetPinned = FullScreenCanvasWidget.Pin();
	if (Widget == nullptr || World == nullptr || FullScreenWidgetPinned.IsValid())
	{
		return false;
	}
	
	const TSharedRef<SConstraintCanvas> FullScreenCanvas = SNew(SConstraintCanvas);
	FullScreenCanvas->AddSlot()
		.Offset(FMargin(0, 0, 0, 0))
		.Anchors(FAnchors(0, 0, 1, 1))
		.Alignment(FVector2D(0, 0))
		[
			SNew(SDPIScaler)
			.DPIScale(MoveTemp(InDPIScale))
			[
				Widget->TakeWidget()
			]
		];

	
	UGameViewportClient* ViewportClient = World->GetGameViewport();
	const bool bCanUseGameViewport = ViewportClient && World->IsGameWorld();
	if (bCanUseGameViewport)
	{
		FullScreenCanvasWidget = FullScreenCanvas;
		ViewportClient->AddViewportWidgetContent(FullScreenCanvas);
		return true;
	}

#if WITH_EDITOR
	const TSharedPtr<FSceneViewport> PinnedTargetViewport = EditorTargetViewport.Pin();
	for (FLevelEditorViewportClient* Client : GEditor->GetLevelViewportClients())
	{
		const TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(Client->GetEditorViewportWidget());
		if (LevelViewport.IsValid() && LevelViewport->GetSceneViewport() == PinnedTargetViewport)
		{
			LevelViewport->AddOverlayWidget(FullScreenCanvas);
			FullScreenCanvasWidget = FullScreenCanvas;
			OverlayWidgetLevelViewport = LevelViewport;
			return true;
		}
	}
#endif

	return false;
}

void FViewportWidgetOverlay_Viewport::Hide(UWorld* World)
{
	TSharedPtr<SConstraintCanvas> FullScreenWidgetPinned = FullScreenCanvasWidget.Pin();
	if (FullScreenWidgetPinned.IsValid())
	{
		// Remove from Viewport and Fullscreen, in case the settings changed before we had the chance to hide.
		UGameViewportClient* ViewportClient = World ? World->GetGameViewport() : nullptr;
		if (ViewportClient)
		{
			ViewportClient->RemoveViewportWidgetContent(FullScreenWidgetPinned.ToSharedRef());
		}

#if WITH_EDITOR
		if (const TSharedPtr<SLevelViewport> OverlayWidgetLevelViewportPinned = OverlayWidgetLevelViewport.Pin())
		{
			OverlayWidgetLevelViewportPinned->RemoveOverlayWidget(FullScreenWidgetPinned.ToSharedRef());
		}
		OverlayWidgetLevelViewport.Reset();
#endif

		FullScreenCanvasWidget.Reset();
	}
}

/////////////////////////////////////////////////////
// FViewportWidgetOverlay_PostProcess



/////////////////////////////////////////////////////
// UVPFullScreenUserWidget

UViewportWidgetOverlay::UViewportWidgetOverlay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CurrentDisplayType(EViewportWidgetOverlay_DisplayType::Inactive)
	, bDisplayRequested(false)
{
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> PostProcessMaterial_Finder(TEXT("/ViewportWidgetOverlay/Materials/WidgetPostProcessMaterial"));
	PostProcessDisplayTypeWithBlendMaterial.PostProcessMaterial = PostProcessMaterial_Finder.Object;
}

void UViewportWidgetOverlay::BeginDestroy()
{
	Hide();
	Super::BeginDestroy();
}

bool UViewportWidgetOverlay::ShouldDisplay(UWorld* InWorld) const
{
#if UE_SERVER
	return false;
#else
	if (GUsingNullRHI || HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject) || IsRunningDedicatedServer())
	{
		return false;
	}

	return GetDisplayType(InWorld) != EViewportWidgetOverlay_DisplayType::Inactive;
#endif //!UE_SERVER
}

EViewportWidgetOverlay_DisplayType UViewportWidgetOverlay::GetDisplayType(UWorld* InWorld) const
{
	if (InWorld)
	{
		if (InWorld->WorldType == EWorldType::Game)
		{
			return GameDisplayType;
		}
#if WITH_EDITOR
		else if (InWorld->WorldType == EWorldType::PIE)
		{
			return PIEDisplayType;
		}
		else if (InWorld->WorldType == EWorldType::Editor)
		{
			return EditorDisplayType;
		}
#endif // WITH_EDITOR
	}
	return EViewportWidgetOverlay_DisplayType::Inactive;
}

bool UViewportWidgetOverlay::IsDisplayed() const
{
	return CurrentDisplayType != EViewportWidgetOverlay_DisplayType::Inactive;
}

void UViewportWidgetOverlay::SetWidgetClass(TSubclassOf<UUserWidget> InWidgetClass)
{
	if (ensure(!IsDisplayed()))
	{
		WidgetClass = InWidgetClass;
	}
}

FViewportWidgetOverlay_PostProcessBase* UViewportWidgetOverlay::GetPostProcessDisplayTypeSettingsFor(EViewportWidgetOverlay_DisplayType Type)
{
	return const_cast<FViewportWidgetOverlay_PostProcessBase*>(const_cast<const UViewportWidgetOverlay*>(this)->GetPostProcessDisplayTypeSettingsFor(Type));
}

const FViewportWidgetOverlay_PostProcessBase* UViewportWidgetOverlay::GetPostProcessDisplayTypeSettingsFor(EViewportWidgetOverlay_DisplayType Type) const
{
	switch (Type)
	{
	case EViewportWidgetOverlay_DisplayType::PostProcessWithBlendMaterial: return &PostProcessDisplayTypeWithBlendMaterial;
	case EViewportWidgetOverlay_DisplayType::PostProcessSceneViewExtension: return &PostProcessWithSceneViewExtensions;
		
	case EViewportWidgetOverlay_DisplayType::Inactive: // Fall-through
	case EViewportWidgetOverlay_DisplayType::Viewport: 
		ensureMsgf(false, TEXT("GetPostProcessDisplayTypeSettingsFor should only be called with PostProcessWithBlendMaterial or PostProcessSceneViewExtension"));
		break;
	default:
		checkNoEntry();
	}
	return nullptr;
}

bool UViewportWidgetOverlay::Display(UWorld* InWorld)
{
	bDisplayRequested = true;
	World = InWorld;

#if WITH_EDITOR
	if (!EditorTargetViewport.IsValid() && !World->IsGameWorld())
	{
		UE_LOGF(LogViewportWidgetOverlay, Log, "No TargetViewport set. Defaulting to FLevelEditorModule::GetFirstActiveLevelViewport.")
		if (FModuleManager::Get().IsModuleLoaded(NAME_LevelEditorName))
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(NAME_LevelEditorName);
			const TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
			EditorTargetViewport = ActiveLevelViewport ? ActiveLevelViewport->GetSharedActiveViewport() : nullptr;
		}

		if (!EditorTargetViewport.IsValid())
		{
			UE_LOGF(LogViewportWidgetOverlay, Error, "FLevelEditorModule::GetFirstActiveLevelViewport found no level viewport. UVPFullScreenUserWidget will not display.")
			return false;
		}
	}
	
	// Make sure that each display type has also received the EditorTargetViewport
	SetEditorTargetViewport(EditorTargetViewport);
#endif

	bool bWasAdded = false;
	if (InWorld && WidgetClass && ShouldDisplay(InWorld) && CurrentDisplayType == EViewportWidgetOverlay_DisplayType::Inactive)
	{
		const bool bCreatedWidget = InitWidget();
		if (!bCreatedWidget)
		{
			UE_LOGF(LogViewportWidgetOverlay, Error, "Failed to create subwidget for UVPFullScreenUserWidget.");
			return false;
		}
		
		CurrentDisplayType = GetDisplayType(InWorld);
		TAttribute<float> GetDpiScaleAttribute = TAttribute<float>::CreateLambda([WeakThis = TWeakObjectPtr<UViewportWidgetOverlay>(this)]()
		{
			return WeakThis.IsValid() ? WeakThis->GetViewportDPIScale() : 1.f;
		});
		if (CurrentDisplayType == EViewportWidgetOverlay_DisplayType::Viewport)
		{
			bWasAdded = ViewportDisplayType.Display(InWorld, Widget, MoveTemp(GetDpiScaleAttribute));
		}
		else if (CurrentDisplayType == EViewportWidgetOverlay_DisplayType::PostProcessWithBlendMaterial)
		{
			bWasAdded = PostProcessDisplayTypeWithBlendMaterial.Display(InWorld, Widget, MoveTemp(GetDpiScaleAttribute));
		}
		else if (CurrentDisplayType == EViewportWidgetOverlay_DisplayType::PostProcessSceneViewExtension)
		{
			bWasAdded = PostProcessWithSceneViewExtensions.Display(InWorld, Widget, MoveTemp(GetDpiScaleAttribute));
		}

		if (bWasAdded)
		{
			FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &UViewportWidgetOverlay::OnLevelRemovedFromWorld);
			FWorldDelegates::OnWorldCleanup.AddUObject(this, &UViewportWidgetOverlay::OnWorldCleanup);

			VPFullScreenUserWidgetPrivate::FWorldCleanupListener::Get()->AddWidget(this);
		}
	}

	return bWasAdded;
}

void UViewportWidgetOverlay::Hide()
{
	bDisplayRequested = false;

	if (CurrentDisplayType != EViewportWidgetOverlay_DisplayType::Inactive)
	{
		ReleaseWidget();

		UWorld* WorldInstance = World.Get();
		if (CurrentDisplayType == EViewportWidgetOverlay_DisplayType::Viewport)
		{
			ViewportDisplayType.Hide(WorldInstance);
		}
		else if (CurrentDisplayType == EViewportWidgetOverlay_DisplayType::PostProcessWithBlendMaterial)
		{
			PostProcessDisplayTypeWithBlendMaterial.Hide(WorldInstance);
		}
		else if (CurrentDisplayType == EViewportWidgetOverlay_DisplayType::PostProcessSceneViewExtension)
		{
			PostProcessWithSceneViewExtensions.Hide(WorldInstance);
		}
		CurrentDisplayType = EViewportWidgetOverlay_DisplayType::Inactive;
	}

	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
	VPFullScreenUserWidgetPrivate::FWorldCleanupListener::Get()->RemoveWidget(this);
	World.Reset();
}

void UViewportWidgetOverlay::Tick(float DeltaSeconds)
{
	if (CurrentDisplayType != EViewportWidgetOverlay_DisplayType::Inactive)
	{
		UWorld* CurrentWorld = World.Get();
		if (CurrentWorld == nullptr)
		{
			Hide();
		}
		else if (CurrentDisplayType == EViewportWidgetOverlay_DisplayType::PostProcessWithBlendMaterial)
		{
			PostProcessDisplayTypeWithBlendMaterial.Tick(CurrentWorld, DeltaSeconds);
		}
		else if (CurrentDisplayType == EViewportWidgetOverlay_DisplayType::PostProcessSceneViewExtension)
		{
			PostProcessWithSceneViewExtensions.Tick(CurrentWorld, DeltaSeconds);
		}
	}
}

void UViewportWidgetOverlay::SetDisplayTypes(EViewportWidgetOverlay_DisplayType InEditorDisplayType, EViewportWidgetOverlay_DisplayType InGameDisplayType, EViewportWidgetOverlay_DisplayType InPIEDisplayType)
{
	EditorDisplayType = InEditorDisplayType;
	GameDisplayType = InGameDisplayType;
	PIEDisplayType = InPIEDisplayType;
}

void UViewportWidgetOverlay::SetCustomPostProcessSettingsSource(TWeakObjectPtr<UObject> InCustomPostProcessSettingsSource)
{
	PostProcessDisplayTypeWithBlendMaterial.SetCustomPostProcessSettingsSource(InCustomPostProcessSettingsSource);
}

#if WITH_EDITOR
void UViewportWidgetOverlay::SetEditorTargetViewport(TWeakPtr<FSceneViewport> InTargetViewport)
{
	EditorTargetViewport = InTargetViewport;
	ViewportDisplayType.EditorTargetViewport = InTargetViewport;
	PostProcessDisplayTypeWithBlendMaterial.EditorTargetViewport = InTargetViewport;
	PostProcessWithSceneViewExtensions.EditorTargetViewport = InTargetViewport;
}

void UViewportWidgetOverlay::ResetEditorTargetViewport()
{
	EditorTargetViewport.Reset();
	ViewportDisplayType.EditorTargetViewport.Reset();
	PostProcessDisplayTypeWithBlendMaterial.EditorTargetViewport.Reset();
	PostProcessWithSceneViewExtensions.EditorTargetViewport.Reset();
}
#endif

bool UViewportWidgetOverlay::InitWidget()
{
	const bool bCanCreate = !Widget && WidgetClass && ensure(World.Get()) && FSlateApplication::IsInitialized();
	if (!bCanCreate)
	{
		return false;
	}

	// Could fail e.g. if the class has been marked deprecated or abstract.
	Widget = CreateWidget(World.Get(), WidgetClass);
	UE_CLOGF(!Widget, LogViewportWidgetOverlay, Warning, "Failed to create widget with class %ls. Review the log for more info.", *WidgetClass->GetPathName())
	if (Widget)
	{
		Widget->SetFlags(RF_Transient);
	}

	return Widget != nullptr;
}

void UViewportWidgetOverlay::ReleaseWidget()
{
	Widget = nullptr;
}

void UViewportWidgetOverlay::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	// If the InLevel is invalid, then the entire world is about to disappear.
	//Hide the widget to clear the memory and reference to the world it may hold.
	if (InLevel == nullptr && InWorld && InWorld == World.Get())
	{
		Hide();
	}
}

void UViewportWidgetOverlay::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	if (IsDisplayed() && World == InWorld)
	{
		Hide();
	}
}

FVector2D UViewportWidgetOverlay::FindSceneViewportSize()
{
	ensure(World.IsValid());
	
	const UWorld* CurrentWorld = World.Get();
	const bool bIsPlayWorld = CurrentWorld && (CurrentWorld->WorldType == EWorldType::Game || CurrentWorld->WorldType == EWorldType::PIE); 
	if (bIsPlayWorld)
	{
		if (UGameViewportClient* ViewportClient = World->GetGameViewport())
		{
			FVector2D OutSize;
			ViewportClient->GetViewportSize(OutSize);
			return OutSize;
		}
	}

#if WITH_EDITOR
	if (const TSharedPtr<FSceneViewport> TargetViewportPin = EditorTargetViewport.Pin())
	{
		return TargetViewportPin->GetSize();
	}
#endif

	ensureMsgf(false, TEXT(
		"FindSceneViewportSize failed. Likely Hide() was called (making World = nullptr) or EditorTargetViewport "
		"reset externally (possibly as part of Hide()). After Hide() is called all widget code should stop calling "
		"FindSceneViewportSize. Investigate whether something was not cleaned up correctly!"
		)
	);
	return FVector2d::ZeroVector;
}

float UViewportWidgetOverlay::GetViewportDPIScale()
{
	float UIScale = 1.0f;
	float PlatformScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(10.0f, 10.0f);

	UWorld* CurrentWorld = World.Get();
	if ((CurrentDisplayType == EViewportWidgetOverlay_DisplayType::Viewport) && CurrentWorld && (CurrentWorld->WorldType == EWorldType::Game || CurrentWorld->WorldType == EWorldType::PIE))
	{
		// If we are in Game or PIE in Viewport display mode, the GameLayerManager will scale correctly so just return the Platform Scale
		UIScale = PlatformScale;
	}
	else
	{
		// Otherwise when in Editor mode, the editor automatically scales to the platform size, so we only care about the UI scale
		const FIntPoint ViewportSize = FindSceneViewportSize().IntPoint();
		UIScale = GetDefault<UUserInterfaceSettings>()->GetDPIScaleBasedOnSize(ViewportSize);
	}

	return UIScale;
}

#if WITH_EDITOR

void UViewportWidgetOverlay::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* Property = PropertyChangedEvent.MemberProperty;

	if (Property && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		static FName NAME_WidgetClass = GET_MEMBER_NAME_CHECKED(UViewportWidgetOverlay, WidgetClass);
		static FName NAME_EditorDisplayType = GET_MEMBER_NAME_CHECKED(UViewportWidgetOverlay, EditorDisplayType);
		static FName NAME_PostProcessMaterial = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, PostProcessMaterial);
		static FName NAME_WidgetDrawSize = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, WidgetDrawSize);
		static FName NAME_WindowFocusable = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, bWindowFocusable);
		static FName NAME_WindowVisibility = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, WindowVisibility);
		static FName NAME_ReceiveHardwareInput = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, bReceiveHardwareInput);
		static FName NAME_RenderTargetBackgroundColor = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, RenderTargetBackgroundColor);
		static FName NAME_RenderTargetBlendMode = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, RenderTargetBlendMode);
		static FName NAME_PostProcessTintColorAndOpacity = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, PostProcessTintColorAndOpacity);
		static FName NAME_PostProcessOpacityFromTexture = GET_MEMBER_NAME_CHECKED(FViewportWidgetOverlay_PostProcess, PostProcessOpacityFromTexture);

		if (Property->GetFName() == NAME_WidgetClass
			|| Property->GetFName() == NAME_EditorDisplayType
			|| Property->GetFName() == NAME_PostProcessMaterial
			|| Property->GetFName() == NAME_WidgetDrawSize
			|| Property->GetFName() == NAME_WindowFocusable
			|| Property->GetFName() == NAME_WindowVisibility
			|| Property->GetFName() == NAME_ReceiveHardwareInput
			|| Property->GetFName() == NAME_RenderTargetBackgroundColor
			|| Property->GetFName() == NAME_RenderTargetBlendMode
			|| Property->GetFName() == NAME_PostProcessTintColorAndOpacity
			|| Property->GetFName() == NAME_PostProcessOpacityFromTexture)
		{
			bool bWasRequestedDisplay = bDisplayRequested;
			UWorld* CurrentWorld = World.Get();
			Hide();
			if (bWasRequestedDisplay && CurrentWorld)
			{
				Display(CurrentWorld);
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

#undef LOCTEXT_NAMESPACE
