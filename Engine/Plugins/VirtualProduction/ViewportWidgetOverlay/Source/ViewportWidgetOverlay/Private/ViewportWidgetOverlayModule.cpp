// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportWidgetOverlayModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"

namespace UE::ViewportWidgetOverlay
{
void FViewportWidgetOverlayModule::StartupModule()
{
	const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("ViewportWidgetOverlay"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/ViewportWidgetOverlay"), PluginShaderDir);
}

void FViewportWidgetOverlayModule::ShutdownModule()
{}

IMPLEMENT_MODULE(FViewportWidgetOverlayModule, ViewportWidgetOverlay);
}
