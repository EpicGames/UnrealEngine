// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportWidgetOverlay_PostProcessBase.h"
#include "ViewportWidgetOverlay_PostProcess.generated.h"

class AActor;
class UPostProcessComponent;

#define UE_API VIEWPORTWIDGETOVERLAY_API

/**
 * Renders widget by adding it as a blend material.
 */
USTRUCT()
struct FViewportWidgetOverlay_PostProcess : public FViewportWidgetOverlay_PostProcessBase
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Composure layers are no longer supported on the VP Full screen user widget.")		
	UPROPERTY()
	TArray<TObjectPtr<AActor>> ComposureLayerTargets;
#endif

	UE_API FViewportWidgetOverlay_PostProcess() = default;
	UE_API FViewportWidgetOverlay_PostProcess(const FViewportWidgetOverlay_PostProcess&) = default;
	UE_API FViewportWidgetOverlay_PostProcess(FViewportWidgetOverlay_PostProcess&&) = default;
	UE_API FViewportWidgetOverlay_PostProcess& operator=(const FViewportWidgetOverlay_PostProcess&) = default;
	UE_API FViewportWidgetOverlay_PostProcess& operator=(FViewportWidgetOverlay_PostProcess&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_API void SetCustomPostProcessSettingsSource(TWeakObjectPtr<UObject> InCustomPostProcessSettingsSource);

	UE_API bool Display(UWorld* World, UUserWidget* Widget, TAttribute<float> InDPIScale);
	
	UE_DEPRECATED(5.7, "bInRenderToTextureOnly parameter is deprecated and use method call without that parameter.")
	bool Display(UWorld* World, UUserWidget* Widget, bool bInRenderToTextureOnly, TAttribute<float> InDPIScale)
	{
		return Display(World, Widget, InDPIScale);
	}
	
	UE_API virtual void Hide(UWorld* World) override;
	UE_API void Tick(UWorld* World, float DeltaSeconds);

private:
	
	/** Post process component used to add the material to the post process chain. */
	UPROPERTY(Transient)
	TObjectPtr<UPostProcessComponent> PostProcessComponent = nullptr;
	
	/** The dynamic instance of the material that the render target is attached to. */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> PostProcessMaterialInstance;

	/**
	 * Optional. Some object that contains a FPostProcessSettings property. These settings will be used for PostProcessMaterialInstance.
	 * E.g. VCam uses this to use post process from a specific cine camera.
	 */
	TWeakObjectPtr<UObject> CustomPostProcessSettingsSource;
	
	bool InitPostProcessComponent(UWorld* World);
	bool UpdateTargetPostProcessSettingsWithMaterial();
	void ReleasePostProcessComponent();
	
	virtual bool OnRenderTargetInited() override;

	FPostProcessSettings* GetPostProcessSettings() const;
};

#undef UE_API
