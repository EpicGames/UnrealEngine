// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/WeakObjectPtrFwd.h"
#include "Components/WidgetComponent.h"
#include "ViewportWidgetOverlay_PostProcessBase.generated.h"

class FSceneViewport;
class FWidgetRenderer;
class SVirtualWindow;
class SViewport;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTextureRenderTarget2D;
class UUserWidget;
class UWorld;

namespace UE::VPUtilities::Private { class FVPWidgetPostProcessHitTester; }

#define UE_API VIEWPORTWIDGETOVERLAY_API

/**
 * Implements shared widget initialization logic.
 */
USTRUCT()
struct FViewportWidgetOverlay_PostProcessBase
{
	GENERATED_BODY()
	
	/**
	 * Post process material used to display the widget.
	 * SlateUI [Texture]
	 * TintColorAndOpacity [Vector]
	 * OpacityFromTexture [Scalar]
	 */
	UPROPERTY(EditAnywhere, Category = PostProcess)
	TObjectPtr<UMaterialInterface> PostProcessMaterial;

	/** Tint color and opacity for this component. */
	UPROPERTY(EditAnywhere, Category = PostProcess)
	FLinearColor PostProcessTintColorAndOpacity;

	/** Sets the amount of opacity from the widget's UI texture to use when rendering the translucent or masked UI to the viewport (0.0-1.0). */
	UPROPERTY(EditAnywhere, Category = PostProcess, meta = (ClampMin = 0.0f, ClampMax = 1.0f))
	float PostProcessOpacityFromTexture;
	
	/** Whether to overwrite the size of the rendered widget. */
	UPROPERTY(EditAnywhere, Category = PostProcess, meta=(InlineEditConditionToggle))
	bool bUseWidgetDrawSize;
	
	/** The size of the rendered widget. */
	UPROPERTY(EditAnywhere, Category = PostProcess, meta=(EditCondition= bWidgetDrawSize))
	FIntPoint WidgetDrawSize;
	
	/** Is the virtual window created to host the widget focusable? */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PostProcess)
	bool bWindowFocusable;

	/** The visibility of the virtual window created to host the widget. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PostProcess)
	EWindowVisibility WindowVisibility;

	/** Register with the viewport for hardware input from the mouse and keyboard. It can and will steal focus from the viewport. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category= PostProcess)
	bool bReceiveHardwareInput;

	/** The background color of the render target */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PostProcess)
	FLinearColor RenderTargetBackgroundColor;

	/** The blend mode for the widget. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = PostProcess)
	EWidgetBlendMode RenderTargetBlendMode;

	/** The target to which the user widget is rendered. */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> WidgetRenderTarget;

#if WITH_EDITOR
	/**
	 * The viewport to use for displaying.
	 * 
	 * Defaults to GetFirstActiveLevelViewport().
	 */
	TWeakPtr<FSceneViewport> EditorTargetViewport;
#endif

	UE_API FViewportWidgetOverlay_PostProcessBase();
	UE_API virtual ~FViewportWidgetOverlay_PostProcessBase() = default;

	/** Deinits this output type */
	UE_API virtual void Hide(UWorld* World);
	
	UE_API TSharedPtr<SVirtualWindow> GetSlateWindow() const;

protected:

	UE_API bool CreateRenderer(UWorld* World, UUserWidget* Widget, TAttribute<float> InDPIScale);
	UE_API virtual void ReleaseRenderer();
	UE_API void TickRenderer(UWorld* World, float DeltaSeconds);
	
private:
	
	/** Helper class for drawing widgets to a render target. */
	FWidgetRenderer* WidgetRenderer;
	
	/** The size of the rendered widget */
	FIntPoint CurrentWidgetDrawSize;
	
	/** The slate viewport we are registered to. */
	TWeakPtr<SViewport> ViewportWidget;
	/** The slate window that contains the user widget content. */
	TSharedPtr<SVirtualWindow> SlateWindow;
	
	/** Hit tester when we want the hardware input. */
	TSharedPtr<UE::VPUtilities::Private::FVPWidgetPostProcessHitTester> CustomHitTestPath;

	/** Creates the post process material and sets up its parameters. */
	virtual bool OnRenderTargetInited() { return true; };
	
	/** Determines widget size depending on the viewport type (PIE / Game) */
	FIntPoint CalculateWidgetDrawSize(UWorld* World);
	bool IsTextureSizeValid(FIntPoint Size) const;

	/** Starts detecting input to viewport and relays it to the user widget. */
	void RegisterHitTesterWithViewport(UWorld* World);
	void UnRegisterHitTesterWithViewport();
	/** Gets the viewport to register the hit tester with. */
	TSharedPtr<SViewport> GetViewport(UWorld* World) const;
	/** Determines the DPI to apply for clicks depending on the type of viewport (PIE / Game). */
	float GetDPIScaleForPostProcessHitTester(TWeakObjectPtr<UWorld> World) const;
};

#undef UE_API
