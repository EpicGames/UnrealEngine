// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportWidgetOverlay_PostProcessBase.h"
#include "SceneViewExtensionContext.h"
#include "ViewportWidgetOverlay_PostProcessWithSVE.generated.h"

class FSceneViewExtensionBase;
class UMaterialInterface;
class UViewportWidgetOverlay;

#define UE_API VIEWPORTWIDGETOVERLAY_API

/**
 * Renders widget in post process phase by using Scene View Extensions (SVE).
 */
USTRUCT()
struct FViewportWidgetOverlay_PostProcessWithSVE : public FViewportWidgetOverlay_PostProcessBase
{
	GENERATED_BODY()

	UE_API bool Display(UWorld* World, UUserWidget* Widget, TAttribute<float> InDPIScale);
	UE_API void Hide(UWorld* World);
	UE_API void Tick(UWorld* World, float DeltaSeconds);

	/**
	 * Registers a functor that will determine whether the SVE will be applied for a given frame.
	 * The IsActive functor should only ever return false. If it is fine to render, it should return an empty optional.
	 *
	 * Functors may be registered before or after the widget is displayed that is before Display(...) is called.
	 * All functors are not retained when Hide(...) is called: they are removed
	 */
	UE_API void RegisterIsActiveFunctor(FSceneViewExtensionIsActiveFunctor IsActiveFunctor);

private:

	/** Implements the rendering side */
	TSharedPtr<FSceneViewExtensionBase, ESPMode::ThreadSafe> SceneViewExtension;

	/** Functors registered before we were displayed. */
	TArray<FSceneViewExtensionIsActiveFunctor> IsActiveFunctorsToRegister;
};

#undef UE_API