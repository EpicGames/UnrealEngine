// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "Components/WidgetComponent.h"
#include "Misc/ViewportWidgetOverlay_PostProcess.h"
#include "Misc/ViewportWidgetOverlay_PostProcessWithSVE.h"
#include "ViewportWidgetOverlay.generated.h"

class FSceneViewport;
class FWidgetRenderer;
class FVPWidgetPostProcessHitTester;
class SConstraintCanvas;
class SVirtualWindow;
class SViewport;
class SWidget;
class ULevel;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UPostProcessComponent;
class UTextureRenderTarget2D;
class UWorld;

#if WITH_EDITOR
class SLevelViewport;
#endif

#define UE_API VIEWPORTWIDGETOVERLAY_API

UENUM(BlueprintType)
enum class EViewportWidgetOverlay_DisplayType : uint8
{
	/** Do not display. */
	Inactive,
	/** Display on a game viewport. */
	Viewport,
	/** Display by adding post process material via post process volume settings. Widget added only over area rendered by anamorphic squeeze. */
	PostProcessWithBlendMaterial,

	Composure = PostProcessWithBlendMaterial UMETA(Deprecated, DisplayName = "DEPRECATED_Composure"),
		
	/** Display by adding post process material via SceneViewExtensions. Widget added over entire viewport ignoring anamorphic squeeze.  */
	PostProcessSceneViewExtension,
};


USTRUCT()
struct FViewportWidgetOverlay_Viewport
{
	GENERATED_BODY()

	bool Display(UWorld* World, UUserWidget* Widget, TAttribute<float> InDPIScale);
	void Hide(UWorld* World);

#if WITH_EDITOR
	/**
	 * The viewport to use for displaying.
	 * Defaults to GetFirstActiveLevelViewport().
	 */
	TWeakPtr<FSceneViewport> EditorTargetViewport;
#endif

private:

	/** Constraint widget that contains the widget we want to display. */
	TWeakPtr<SConstraintCanvas> FullScreenCanvasWidget;

#if WITH_EDITOR
	/** Level viewport the widget was added to. */
	TWeakPtr<SLevelViewport> OverlayWidgetLevelViewport;
#endif
};

/**
 * Will set the Widgets on a viewport either by Widgets are first rendered to a render target, then that render target is displayed in the world.
 */
UCLASS(meta=(ShowOnlyInnerProperties))
class UViewportWidgetOverlay : public UObject
{
	GENERATED_BODY()
public:
	
	UE_API UViewportWidgetOverlay(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject interface
	UE_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	UE_API virtual bool Display(UWorld* World);
	UE_API virtual void Hide();
	UE_API virtual void Tick(float DeltaTime);

	/** Sets the way the widget is supposed to be displayed */
	UE_API void SetDisplayTypes(EViewportWidgetOverlay_DisplayType InEditorDisplayType, EViewportWidgetOverlay_DisplayType InGameDisplayType, EViewportWidgetOverlay_DisplayType InPIEDisplayType);
	
	/**
	 * If using EVPWidgetDisplayType::PostProcessWithBlendMaterial, you can specify a custom post process settings that should be modified.
	 * By default, a new post process component is added to AWorldSettings.
	 *
	 * @param InCustomPostProcessSettingsSource An object containing a FPostProcessSettings UPROPERTY()
	 */
	UE_API void SetCustomPostProcessSettingsSource(TWeakObjectPtr<UObject> InCustomPostProcessSettingsSource);
	
	UE_API bool ShouldDisplay(UWorld* World) const;
	UE_API EViewportWidgetOverlay_DisplayType GetDisplayType(UWorld* World) const;
	UE_API bool IsDisplayed() const;
	
	/** Get a pointer to the inner widget. Note: This should not be stored! */
	UUserWidget* GetWidget() const { return Widget; };

	/** Gets the widget class being displayed */
	TSubclassOf<UUserWidget> GetWidgetClass() const { return WidgetClass; }
	/** Sets the widget class to be displayed. This must be called while not widget is being displayed - the class will not be updated while IsDisplayed().  */
	UE_API void SetWidgetClass(TSubclassOf<UUserWidget> InWidgetClass);

	constexpr static bool DoesDisplayTypeUsePostProcessSettings(EViewportWidgetOverlay_DisplayType Type) { return Type == EViewportWidgetOverlay_DisplayType::PostProcessSceneViewExtension || Type == EViewportWidgetOverlay_DisplayType::PostProcessWithBlendMaterial; }
	/** Returns the base post process settings for the given type if DoesDisplayTypeUsePostProcessSettings(Type) returns true. */
	UE_API FViewportWidgetOverlay_PostProcessBase* GetPostProcessDisplayTypeSettingsFor(EViewportWidgetOverlay_DisplayType Type);
	FViewportWidgetOverlay_PostProcess& GetPostProcessDisplayTypeWithBlendMaterialSettings() { return PostProcessDisplayTypeWithBlendMaterial; }
	
	UE_DEPRECATED(5.7, "Composure is no longer support for VPFullScreenUserWidgets")
	FViewportWidgetOverlay_PostProcess& GetComposureDisplayTypeSettings() { return PostProcessDisplayTypeWithBlendMaterial; }
	
	FViewportWidgetOverlay_PostProcessWithSVE& GetPostProcessDisplayTypeWithSceneViewExtensionsSettings() { return PostProcessWithSceneViewExtensions; }
	/** Returns the base post process settings for the given type if DoesDisplayTypeUsePostProcessSettings(Type) returns true. */
	UE_API const FViewportWidgetOverlay_PostProcessBase* GetPostProcessDisplayTypeSettingsFor(EViewportWidgetOverlay_DisplayType Type) const;
	const FViewportWidgetOverlay_PostProcess& GetPostProcessDisplayTypeWithBlendMaterialSettings() const { return PostProcessDisplayTypeWithBlendMaterial; }

	UE_DEPRECATED(5.7, "Composure is no longer support for VPFullScreenUserWidgets")
	const FViewportWidgetOverlay_PostProcess& GetComposureDisplayTypeSettings() const { return PostProcessDisplayTypeWithBlendMaterial; }
	const FViewportWidgetOverlay_PostProcessWithSVE& GetPostProcessDisplayTypeWithSceneViewExtensionsSettings() const { return PostProcessWithSceneViewExtensions; }

#if WITH_EDITOR
	/**
	 * Sets the TargetViewport to use on both the Viewport and the PostProcess class.
	 * 
	 * Overrides the viewport to use for displaying.
	 * Defaults to GetFirstActiveLevelViewport().
	 */
	UE_API void SetEditorTargetViewport(TWeakPtr<FSceneViewport> InTargetViewport);
	/** Resets the TargetViewport  */
	UE_API void ResetEditorTargetViewport();
#endif

protected:
	
	/** The class of User Widget to create and display an instance of */
	UPROPERTY(EditAnywhere, Category = "User Interface")
	TSubclassOf<UUserWidget> WidgetClass;
	
	/** The display type when the world is an editor world. */
	UPROPERTY(EditAnywhere, Category = "User Interface")
	EViewportWidgetOverlay_DisplayType EditorDisplayType;

	/** The display type when the world is a game world. */
	UPROPERTY(EditAnywhere, Category = "User Interface")
	EViewportWidgetOverlay_DisplayType GameDisplayType;

	/** The display type when the world is a PIE world. */
	UPROPERTY(EditAnywhere, Category = "User Interface", meta = (DisplayName = "PIE Display Type"))
	EViewportWidgetOverlay_DisplayType PIEDisplayType;

	/** Behavior when the widget should be display by the slate attached to the viewport. */
	UPROPERTY(EditAnywhere, Category = "Viewport", meta = (ShowOnlyInnerProperties))
	FViewportWidgetOverlay_Viewport ViewportDisplayType;

	/** Behavior when the widget is rendered via a post process material via post process volume settings; the widget is added only over area rendered by anamorphic squeeze. */
	UPROPERTY(EditAnywhere, Category = "Post Process (Blend Material)")
	FViewportWidgetOverlay_PostProcess PostProcessDisplayTypeWithBlendMaterial;
	
	/** Behavior when the widget is rendered via a post process material via SceneViewExtensions; the widget is added over entire viewport, ignoring anamorphic squeeze. */
	UPROPERTY(EditAnywhere, Category = "Post Process (Scene View Extensions)")
	FViewportWidgetOverlay_PostProcessWithSVE PostProcessWithSceneViewExtensions;
	
	bool InitWidget();
	void ReleaseWidget();

	FVector2D FindSceneViewportSize();
	float GetViewportDPIScale();

private:
	
	/** The User Widget object displayed and managed by this component */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UUserWidget> Widget;

	/** The world the widget is attached to. */
	UPROPERTY(Transient, DuplicateTransient)
	TWeakObjectPtr<UWorld> World;

	/** How we currently displaying the widget. */
	UPROPERTY(Transient, DuplicateTransient)
	EViewportWidgetOverlay_DisplayType CurrentDisplayType;

	/** The user requested the widget to be displayed. It's possible that some setting are invalid and the widget will not be displayed. */
	bool bDisplayRequested;

#if WITH_EDITOR
	/**
	 * The viewport to use for displaying.
	 * Defaults to GetFirstActiveLevelViewport().
	 */
	TWeakPtr<FSceneViewport> EditorTargetViewport;
#endif
	
	void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);
	void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);
};

#undef UE_API
