// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ViewportWidgetOverlay : ModuleRules
{
	public ViewportWidgetOverlay(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"UMG"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Projects",
				"Renderer",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore"
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"LevelEditor",
					"UnrealEd",
					"ViewportInteraction"
				}
			);
		}
	}
}
