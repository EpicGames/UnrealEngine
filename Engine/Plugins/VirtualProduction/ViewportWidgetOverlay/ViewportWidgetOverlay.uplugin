{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Viewport Widget Overlay",
	"Description": "Adds a utility for overlaying a widget in the viewport.",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"Hidden": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ViewportWidgetOverlay",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	]
}